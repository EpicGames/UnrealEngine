// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkCompression.h"


#include "Misc/App.h"
#include "Misc/Compression.h"
#include "UObject/StructOnScope.h"

#include "HAL/IConsoleManager.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "StructSerializer.h"
#include "StructDeserializer.h"
#include "Backends/CborStructSerializerBackend.h"
#include "Backends/CborStructDeserializerBackend.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkCompression)

namespace UE::LiveLink::Compression
{
	static TAutoConsoleVariable<int32> CVarCompressionType(
		TEXT("LiveLink.SetCompressionType"), 1,
		TEXT("Specify the type of compression to use when serializing data. A value of 0 means compression is off. A value of 1 = Oodle. All other values = Zlib."));

	static TAutoConsoleVariable<int32> CVarCompressionFlags(
		TEXT("LiveLink.SetCompressionFlags"), 0,
		TEXT("Specify the flags to use when compression is enabled. A value of 0 means no flags. A value of 1 favors smaller sizes. Any other value favors faster encoding."));

	int32 GetConsoleVariableCompressionType()
	{
		return CVarCompressionType.GetValueOnAnyThread();
	}

	int32 GetConsoleVariableCompressionFlags()
	{
		return CVarCompressionFlags.GetValueOnAnyThread();
	}

}

namespace PayloadDetail
{

	bool ShouldCompress(const FLiveLinkSerializedFrameData& InPayload, ELiveLinkPayloadCompressionType CompressionType)
	{
		if (CompressionType == ELiveLinkPayloadCompressionType::None)
		{
			return false;
		}
		if (CompressionType == ELiveLinkPayloadCompressionType::Heuristic)
		{
			return UE::LiveLink::Compression::ShouldCompress(InPayload.PayloadSize);
		}
		check(CompressionType == ELiveLinkPayloadCompressionType::Always);
		// Otherwise we are always compressing
		return InPayload.PayloadSize > 0;
	}

	bool TryCompressImpl(const UScriptStruct* InEventType, const void* InEventData, FLiveLinkSerializedFrameData& InOutPayload, ELiveLinkPayloadCompressionType CompressionType)
	{
		InOutPayload.PayloadSize = InOutPayload.PayloadBytes.Bytes.Num();

		// if we serialized something, compress it
		if (ShouldCompress(InOutPayload, CompressionType))
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LiveLink::TryCompressImpl);

			TArray<uint8>& InBytes = InOutPayload.PayloadBytes.Bytes;
			TArray<uint8> OutCompressedData;
			// Compress the result to send on the wire
			FName NamedCompressionAlgo = UE::LiveLink::Compression::GetCompressionAlgorithm();
			ECompressionFlags CompressFlags = UE::LiveLink::Compression::GetCompressionFlags();
			int32 CompressedSize = FCompression::CompressMemoryBound(NamedCompressionAlgo, InOutPayload.PayloadSize, CompressFlags);
			OutCompressedData.SetNumUninitialized(CompressedSize);

			if (FCompression::CompressMemory(NamedCompressionAlgo, OutCompressedData.GetData(), CompressedSize, InBytes.GetData(), InBytes.Num(), CompressFlags))
			{
				OutCompressedData.SetNum(CompressedSize, EAllowShrinking::No);
				InOutPayload.PayloadBytes.Bytes = MoveTemp(OutCompressedData);

				InOutPayload.CompressionMethod = UE::LiveLink::Compression::GetCompressionMethod(NamedCompressionAlgo);
				InOutPayload.CompressionBias = UE::LiveLink::Compression::GetCompressionBias(CompressFlags);
			}
			else
			{
				UE_LOGF(LogTemp, Warning, "Unable to compress data for %ls!", *InEventType->GetName());
				InOutPayload.CompressionMethod = ELiveLinkCompressionMethod::Uncompressed;
			}
		}
		else
		{
			InOutPayload.CompressionMethod = ELiveLinkCompressionMethod::Uncompressed;
		}

		// Since we can support uncompressed or compressed data this is always successful.
		return true;
	}

	static TOptional<TArray<uint8>> DecompressImpl(const FLiveLinkSerializedFrameData& InPayload)
	{
		ELiveLinkCompressionMethod CompressMethod = InPayload.CompressionMethod;
		if (CompressMethod != ELiveLinkCompressionMethod::Uncompressed)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LiveLink::TryDecompress);

			const TArray<uint8>& InBytes = InPayload.PayloadBytes.Bytes;
			TArray<uint8> UncompressedData;
			UncompressedData.SetNumUninitialized(InPayload.PayloadSize);

			ECompressionFlags CompressFlags = UE::LiveLink::Compression::GetCoreCompressionFlags(InPayload.CompressionBias);
			FName CompressType = UE::LiveLink::Compression::GetCompressionAlgorithm(InPayload.CompressionMethod);
			if (FCompression::UncompressMemory(CompressType, UncompressedData.GetData(), UncompressedData.Num(), InBytes.GetData(), InBytes.Num(), CompressFlags))
			{
				return TOptional<TArray<uint8>>(MoveTemp(UncompressedData));
			}
			else
			{
				UE_LOGF(LogTemp, Warning, "Unable to uncompress data for %ls!", *InPayload.PayloadTypeName.ToString());
			}
		}

		return TOptional<TArray<uint8>>{};
	}

	bool SerializeImpl(const UScriptStruct* InSourceEventType, const void* InSourceEventData, FLiveLinkSerializedFrameData& OutSerializedData)
	{
		if (OutSerializedData.SerializationMethod == ELiveLinkPayloadSerializationMethod::Cbor)
		{
			FMemoryWriter Writer(OutSerializedData.PayloadBytes.Bytes);
			FCborStructSerializerBackend Serializer(Writer, EStructSerializerBackendFlags::Default);
			FStructSerializer::Serialize(InSourceEventData, *const_cast<UScriptStruct*>(InSourceEventType), Serializer);
			return !Writer.GetError();
		}

		FMemoryWriter Archive(OutSerializedData.PayloadBytes.Bytes);
		Archive.SetWantBinaryPropertySerialization(true);
		const_cast<UScriptStruct*>(InSourceEventType)->SerializeItem(Archive, (uint8*)InSourceEventData, nullptr);
		return !Archive.GetError();
	}

	bool DeserializeImpl(const UScriptStruct* InTargetEventType, void* InOutTargetEventData, ELiveLinkPayloadSerializationMethod SerializeMethod, const TArray<uint8>& InBytes)
	{
		if (SerializeMethod == ELiveLinkPayloadSerializationMethod::Cbor)
		{
			FMemoryReader Reader(InBytes);
			FCborStructDeserializerBackend Deserializer(Reader);
			return FStructDeserializer::Deserialize(InOutTargetEventData, *const_cast<UScriptStruct*>(InTargetEventType), Deserializer) && !Reader.GetError();
		}

		FMemoryReader Archive(InBytes);
		Archive.SetWantBinaryPropertySerialization(true);
		const_cast<UScriptStruct*>(InTargetEventType)->SerializeItem(Archive, (uint8*)InOutTargetEventData, nullptr);
		return !Archive.GetError();
	}

	bool DeserializeAndDecompress(const UScriptStruct* InTargetEventType, void* InOutTargetEventData, const FLiveLinkSerializedFrameData& InPayload)
	{
		TOptional<TArray<uint8>> DecompressedBytes = DecompressImpl(InPayload);
		ELiveLinkCompressionMethod CompressMethod = InPayload.CompressionMethod;

		if (CompressMethod != ELiveLinkCompressionMethod::Uncompressed && !DecompressedBytes.IsSet())
		{
			return false;
		}
		const TArray<uint8>& ByteStream = DecompressedBytes.IsSet() ? DecompressedBytes.GetValue() : InPayload.PayloadBytes.Bytes;
		return DeserializeImpl(InTargetEventType, InOutTargetEventData, InPayload.SerializationMethod, ByteStream);
	}

	bool SerializePreChecks(const UScriptStruct* InSourceEventType, const void* InSourceEventData, FLiveLinkSerializedFrameData& OutSerializedData)
	{
		OutSerializedData.PayloadSize = 0;
		OutSerializedData.PayloadBytes.Bytes.Reset();

		return InSourceEventType && InSourceEventData;
	}

	bool DeserializePreChecks(const UScriptStruct* InEventType, void* InOutEventData, const FLiveLinkSerializedFrameData& Payload)
	{
		return InEventType && InOutEventData;
	}

} // namespace PayloadDetail

bool FLiveLinkSerializedFrameData::SetPayload(const FStructOnScope& InPayload, ELiveLinkPayloadCompressionType CompressionType)
{
	const UStruct* PayloadStruct = InPayload.GetStruct();
	check(PayloadStruct->IsA<UScriptStruct>());
	return SetPayload((UScriptStruct*)PayloadStruct, InPayload.GetStructMemory(), CompressionType);
}

bool FLiveLinkSerializedFrameData::SetPayload(const UScriptStruct* InPayloadType, const void* InPayloadData, ELiveLinkPayloadCompressionType CompressionType)
{
	check(InPayloadType && InPayloadData);
	PayloadTypeName = *InPayloadType->GetPathName();
	return PayloadDetail::SerializePreChecks(InPayloadType, InPayloadData, *this)
		&& PayloadDetail::SerializeImpl(InPayloadType, InPayloadData, *this)
		&& PayloadDetail::TryCompressImpl(InPayloadType, InPayloadData, *this, CompressionType);
}

bool FLiveLinkSerializedFrameData::GetPayload(FStructOnScope& OutPayload) const
{
	UStruct* PayloadType = nullptr;
	{
		FGCScopeGuard Guard;
		PayloadType = FindObject<UStruct>(nullptr, *PayloadTypeName.ToString());
	}

	if (PayloadType)
	{
		OutPayload.Initialize(PayloadType);
		const UStruct* PayloadStruct = OutPayload.GetStruct();
		check(PayloadStruct->IsA<UScriptStruct>());
		return PayloadDetail::DeserializePreChecks((UScriptStruct*)PayloadStruct, OutPayload.GetStructMemory(), *this)
			&& PayloadDetail::DeserializeAndDecompress((UScriptStruct*)PayloadStruct, OutPayload.GetStructMemory(), *this);
	}
	return false;
}

bool FLiveLinkSerializedFrameData::GetPayload(const UScriptStruct* InPayloadType, void* InOutPayloadData) const
{
	check(InPayloadType && InOutPayloadData);
	return IsTypeChildOf(InPayloadType)
		&& PayloadDetail::DeserializePreChecks((UScriptStruct*)InPayloadType, InOutPayloadData, *this)
		&& PayloadDetail::DeserializeAndDecompress((UScriptStruct*)InPayloadType, InOutPayloadData, *this);
}

bool FLiveLinkSerializedFrameData::IsTypeChildOf(const UScriptStruct* InPayloadType) const
{
	const UStruct* PayloadType = FindObject<UStruct>(nullptr, *PayloadTypeName.ToString());
	return PayloadType && InPayloadType->IsChildOf(PayloadType);
}

