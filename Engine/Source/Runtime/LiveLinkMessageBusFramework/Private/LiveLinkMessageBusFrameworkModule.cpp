// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FLiveLinkMessageBusFrameworkModule : public IModuleInterface
{
public:

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}
};

IMPLEMENT_MODULE(FLiveLinkMessageBusFrameworkModule, LiveLinkMessageBusFramework);
