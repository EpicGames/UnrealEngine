// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkProvider.h"
#include "LiveLinkProviderImpl.h"

#include "Algo/Find.h"
#include "Algo/RemoveIf.h"
#include "Algo/Transform.h"
#include "HAL/PlatformProcess.h"
#include "IMessageContext.h"
#include "LiveLinkCompression.h"
#include "LiveLinkMessages.h"
#include "LiveLinkTypes.h"

#include "Logging/LogMacros.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "Misc/ScopeLock.h"
#include "Roles/LiveLinkAnimationTypes.h"

DEFINE_LOG_CATEGORY_STATIC(LogLiveLinkMessageBus, Warning, All);

static const int32 LIVELINK_SupportedVersion = 2;

template TSharedPtr<ILiveLinkProvider> ILiveLinkProvider::CreateLiveLinkProvider<FLiveLinkProvider>(const FString&,
																									FMessageEndpointBuilder&&);

FName FLiveLinkMessageAnnotation::SubjectAnnotation = TEXT("SubjectName");
FName FLiveLinkMessageAnnotation::RoleAnnotation = TEXT("Role");
FName FLiveLinkMessageAnnotation::OriginalSourceAnnotation = TEXT("OriginalSource");
FName FLiveLinkMessageAnnotation::TopologyModeAnnotation = TEXT("TopologyMode");
FName FLiveLinkMessageAnnotation::CompressedPayloadSupport = TEXT("CompressedPayloadSupport");

static TAutoConsoleVariable<int32> CVarLiveLinkCompressAnimData(
	TEXT("LiveLink.CompressAnimationData"), 1,
	TEXT("Whether to compress animation data that goes through the provider."),
	ECVF_RenderThreadSafe);

// Address that we have had a connection request from
struct FTrackedAddress
{
	FTrackedAddress(FMessageAddress InAddress, bool bInSupportsCompressedPayloads)
		: Address(InAddress)
		, LastHeartbeatTime(FPlatformTime::Seconds())
		, bSupportsCompressedPayloads(bInSupportsCompressedPayloads)
	{}

	FMessageAddress Address;
	double LastHeartbeatTime;
	bool bSupportsCompressedPayloads;
};


// Validate the supplied connection as still active
struct FConnectionValidator
{
	FConnectionValidator()
		: CutOffTime(FPlatformTime::Seconds() - CONNECTION_TIMEOUT)
	{}

	bool operator()(const FTrackedAddress& Connection) const { return Connection.LastHeartbeatTime >= CutOffTime; }

private:
	// How long we give connections before we decide they are dead
	static const double CONNECTION_TIMEOUT;

	// Oldest time that we still deem as active
	const double CutOffTime;
};

const double FConnectionValidator::CONNECTION_TIMEOUT = 10.f;

// Static Subject data that the application has told us about
struct FTrackedStaticData
{
	FTrackedStaticData()
		: SubjectName(NAME_None)
	{}
	FTrackedStaticData(FName InSubjectName, TWeakObjectPtr<UClass> InRoleClass, FLiveLinkStaticDataStruct InStaticData, TMap<FName, FString>&& InAnnotations)
		: SubjectName(InSubjectName), RoleClass(InRoleClass), StaticData(MoveTemp(InStaticData))
		, Annotations(MoveTemp(InAnnotations))
	{}
	FName SubjectName;
	TWeakObjectPtr<UClass> RoleClass;
	FLiveLinkStaticDataStruct StaticData;
	TMap<FName, FString> Annotations;

	bool operator==(FName InSubjectName) const { return SubjectName == InSubjectName; }
};


// Frame Subject data that the application has told us about
struct FTrackedFrameData
{
	FTrackedFrameData()
		: SubjectName(NAME_None)
	{}
	FTrackedFrameData(FName InSubjectName, FLiveLinkFrameDataStruct InFrameData, TMap<FName, FString>&& InAnnotations)
		: SubjectName(InSubjectName), FrameData(MoveTemp(InFrameData))
		, Annotations(MoveTemp(InAnnotations))
	{}
	FName SubjectName;
	FLiveLinkFrameDataStruct FrameData;
	TMap<FName, FString> Annotations;
	bool operator==(FName InSubjectName) const { return SubjectName == InSubjectName; }
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS

// Validate our current connections
void FLiveLinkProvider::ValidateConnections()
{
	FConnectionValidator Validator;

	TArray<FMessageAddress> RemovedConnections;

	// Using SetNumUninitialized because FTrackedAddress does not have a default constructor, resulting in SetNum not
	// compiling (due to the DefaultConstructItems<> usage). Uninitialized is not unsafe here, because we're shrinking.
	ConnectedAddresses.SetNumUninitialized(Algo::RemoveIf(ConnectedAddresses, [this, &Validator, &RemovedConnections](const FTrackedAddress& Address) mutable
	{
		if (!Validator(Address))
	    {
			RemovedConnections.Add(Address.Address);
			return true;
	    }
		return false;
	}));

	if (RemovedConnections.Num() > 0)
	{
		OnConnectionsClosed(RemovedConnections);
		OnConnectionStatusChanged.Broadcast();
	}
}

void FLiveLinkProvider::CloseConnection(FMessageAddress Address)
{
	TSet<FMessageAddress> RemovedConnections;

	{
		FScopeLock Lock(&CriticalSection);
		ConnectedAddresses.SetNumUninitialized(Algo::RemoveIf(ConnectedAddresses, [this, Address, &RemovedConnections](const FTrackedAddress& TrackedAddress) mutable
		{
			if (TrackedAddress.Address == Address)
			{
				RemovedConnections.Add(TrackedAddress.Address);
				return true;
			}
			return false;
		}));
	}

	if (RemovedConnections.Num() > 0)
	{
		OnConnectionsClosed(RemovedConnections.Array());
		OnConnectionStatusChanged.Broadcast();
	}
}

// Get the cached data for the named subject
FTrackedSubject& FLiveLinkProvider::GetTrackedSubject(const FName& SubjectName)
{
	return Subjects.FindOrAdd(SubjectName);
}

// Send hierarchy data for named subject
void FLiveLinkProvider::SendSubject(FName SubjectName, const FTrackedSubject& Subject)
{
	FLiveLinkSubjectDataMessage* SubjectData = FMessageEndpoint::MakeMessage<FLiveLinkSubjectDataMessage>();
	SubjectData->RefSkeleton = Subject.RefSkeleton;
	SubjectData->SubjectName = SubjectName;

	TArray<FMessageAddress> Addresses;
	GetFilteredAddresses(SubjectName, Addresses);

	MessageEndpoint->Send(SubjectData, FLiveLinkSubjectDataMessage::StaticStruct(), EMessageFlags::None, GetAnnotations(), nullptr, Addresses, FTimespan::Zero(), FDateTime::MaxValue());
}

// Send frame data for named subject
void FLiveLinkProvider::SendSubjectFrame(FName SubjectName, const FTrackedSubject& Subject)
{
	FLiveLinkSubjectFrameMessage* SubjectFrame = FMessageEndpoint::MakeMessage<FLiveLinkSubjectFrameMessage>();
	SubjectFrame->Transforms = Subject.Transforms;
	SubjectFrame->SubjectName = SubjectName;
	SubjectFrame->Curves = Subject.Curves;
	SubjectFrame->MetaData = Subject.MetaData;
	SubjectFrame->Time = Subject.Time;

	TArray<FMessageAddress> Addresses;
	GetFilteredAddresses(SubjectName, Addresses);

	MessageEndpoint->Send(SubjectFrame, FLiveLinkSubjectFrameMessage::StaticStruct(), EMessageFlags::None, GetAnnotations(), nullptr, Addresses, FTimespan::Zero(), FDateTime::MaxValue());
}

TPair<UClass*, FLiveLinkStaticDataStruct*> FLiveLinkProvider::GetLastSubjectStaticDataStruct(FName SubjectName)
{
	FScopeLock Lock(&CriticalSection);
	TPair<UClass*, FLiveLinkStaticDataStruct*> Pair = { nullptr, nullptr };

	if (FTrackedStaticData* TrackedStaticData = GetLastSubjectStaticData(SubjectName))
	{
		if (TrackedStaticData->RoleClass.IsValid() && TrackedStaticData->StaticData.IsValid())
		{
			Pair.Key = TrackedStaticData->RoleClass.Get();
			Pair.Value = &TrackedStaticData->StaticData;
		}
	}

	return Pair;
}

// Get the cached data for the named subject
FTrackedStaticData* FLiveLinkProvider::GetLastSubjectStaticData(const FName& SubjectName)
{
	return StaticDatas.FindByKey(SubjectName);
}

FTrackedFrameData* FLiveLinkProvider::GetLastSubjectFrameData(const FName& SubjectName)
{
	return FrameDatas.FindByKey(SubjectName);
}

void FLiveLinkProvider::SetLastSubjectStaticData(FName SubjectName, TSubclassOf<ULiveLinkRole> Role, FLiveLinkStaticDataStruct&& StaticData, TMap<FName, FString>&& SubjectAnnotations)
{
	FTrackedStaticData* Result = StaticDatas.FindByKey(SubjectName);
	if (Result)
	{
		Result->Annotations = MoveTemp(SubjectAnnotations);
		Result->StaticData = MoveTemp(StaticData);
		Result->RoleClass = Role.Get();
	}
	else
	{
		StaticDatas.Emplace(SubjectName, Role.Get(), MoveTemp(StaticData), MoveTemp(SubjectAnnotations));
	}
}

void FLiveLinkProvider::SetLastSubjectFrameData(FName SubjectName, FLiveLinkFrameDataStruct&& FrameData, TMap<FName, FString>&& SubjectAnnotations)
{
	FTrackedFrameData* Result = FrameDatas.FindByKey(SubjectName);
	if (Result)
	{
		Result->FrameData = MoveTemp(FrameData);
		Result->Annotations = MoveTemp(SubjectAnnotations);
	}
	else
	{
		FrameDatas.Emplace(SubjectName, MoveTemp(FrameData), MoveTemp(SubjectAnnotations));
	}
}

// Clear a existing track subject
void FLiveLinkProvider::ClearTrackedSubject(const FName& SubjectName)
{
	Subjects.Remove(SubjectName);

	const int32 FrameIndex = FrameDatas.IndexOfByKey(SubjectName);
	
	if (FrameIndex != INDEX_NONE)
	{
		FrameDatas.RemoveAtSwap(FrameIndex);
	}

	const int32 StaticIndex = StaticDatas.IndexOfByKey(SubjectName);

	if (StaticIndex != INDEX_NONE)
	{
		StaticDatas.RemoveAtSwap(StaticIndex);
	}

	SubjectFrameIds.Remove(SubjectName);
}

FLiveLinkProvider::FLiveLinkProvider(const FString& InProviderName)
	: ProviderName(InProviderName)
	, MachineName(FPlatformProcess::ComputerName())
{
	FMessageEndpointBuilder EndpointBuilder = FMessageEndpoint::Builder(*InProviderName);
	CreateMessageEndpoint(EndpointBuilder);
}

FLiveLinkProvider::FLiveLinkProvider(const FString& InProviderName, FMessageEndpointBuilder&& EndpointBuilder)
	: ProviderName(InProviderName)
	, MachineName(FPlatformProcess::ComputerName())
{
	CreateMessageEndpoint(EndpointBuilder);
}

FLiveLinkProvider::FLiveLinkProvider(const FString& InProviderName, bool bInCreateEndpoint)
	: ProviderName(InProviderName)
	, MachineName(FPlatformProcess::ComputerName())
{
	if (bInCreateEndpoint)
	{
		FMessageEndpointBuilder EndpointBuilder = FMessageEndpoint::Builder(*InProviderName);
    	CreateMessageEndpoint(EndpointBuilder);
	}
}

FLiveLinkProvider::~FLiveLinkProvider()
{
	if (MessageEndpoint.IsValid())
	{
		// Disable the Endpoint message handling since the message could keep it alive a bit.
		MessageEndpoint->Disable();
		MessageEndpoint.Reset();
	}
}

void FLiveLinkProvider::UpdateSubject(const FName& SubjectName, const TArray<FName>& BoneNames, const TArray<int32>& BoneParents)
{
	FScopeLock Lock(&CriticalSection);

	FTrackedSubject& Subject = GetTrackedSubject(SubjectName);
	Subject.RefSkeleton.SetBoneNames(BoneNames);
	Subject.RefSkeleton.SetBoneParents(BoneParents);
	Subject.Transforms.Empty();

	SendSubject(SubjectName, Subject);
}

void FLiveLinkProvider::SendClearSubjectToConnections(FName SubjectName)
{
	TArray<FMessageAddress> MessageAddresses;
	GetFilteredAddresses(SubjectName, MessageAddresses);

	MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FLiveLinkClearSubject>(SubjectName), EMessageFlags::Reliable, GetAnnotations(), nullptr, MessageAddresses, FTimespan::Zero(), FDateTime::MaxValue());
}

FMessageAddress FLiveLinkProvider::GetEndpointAddress() const
{
	return MessageEndpoint->GetAddress();
}

bool FLiveLinkProvider::UpdateSubjectStaticData(const FName SubjectName, TSubclassOf<ULiveLinkRole> Role, FLiveLinkStaticDataStruct&& StaticData, const TMap<FName, FString>& ExtraAnnotations)
{
	FScopeLock Lock(&CriticalSection);

	if (SubjectName == NAME_None || Role.Get() == nullptr)
	{
		return false;
	}

	if (Role->GetDefaultObject<ULiveLinkRole>()->GetStaticDataStruct() != StaticData.GetStruct())
	{
		return false;
	}

	if (GetLastSubjectStaticData(SubjectName) != nullptr)
	{
		ClearSubject(SubjectName);
	}

	ValidateConnections();

	TMap<FName, FString> Annotations;
	Annotations.Add(FLiveLinkMessageAnnotation::SubjectAnnotation, SubjectName.ToString());
	Annotations.Add(FLiveLinkMessageAnnotation::RoleAnnotation, Role->GetName());
	Annotations.Append(ExtraAnnotations);

	if (ConnectedAddresses.Num() > 0)
	{
		TArray<FMessageAddress> Addresses;
		GetFilteredAddresses(SubjectName, Addresses);

		MessageEndpoint->Send(StaticData.CloneData(), const_cast<UScriptStruct*>(StaticData.GetStruct()), EMessageFlags::Reliable, Annotations, nullptr, Addresses, FTimespan::Zero(), FDateTime::MaxValue());
	}

	SetLastSubjectStaticData(SubjectName, Role, MoveTemp(StaticData), MoveTemp(Annotations));

	return true;
}

void FLiveLinkProvider::ClearSubject(const FName& SubjectName)
{
	FScopeLock Lock(&CriticalSection);

	RemoveSubject(SubjectName);
}

void FLiveLinkProvider::RemoveSubject(const FName SubjectName)
{
	FScopeLock Lock(&CriticalSection);

	ClearTrackedSubject(SubjectName);
	SendClearSubjectToConnections(SubjectName);
}

void FLiveLinkProvider::UpdateSubjectFrame(const FName& SubjectName, const TArray<FTransform>& BoneTransforms, const TArray<FLiveLinkCurveElement>& CurveData, double Time)
{
	FScopeLock Lock(&CriticalSection);

	FTrackedSubject& Subject = GetTrackedSubject(SubjectName);

	Subject.Transforms = BoneTransforms;
	Subject.Curves = CurveData;
	Subject.Time = Time;

	SendSubjectFrame(SubjectName, Subject);
}

void FLiveLinkProvider::UpdateSubjectFrame(const FName& SubjectName, const TArray<FTransform>& BoneTransforms, const TArray<FLiveLinkCurveElement>& CurveData,
	const FLiveLinkMetaData& MetaData, double Time)
{
	FScopeLock Lock(&CriticalSection);

	FTrackedSubject& Subject = GetTrackedSubject(SubjectName);

	Subject.Transforms = BoneTransforms;
	Subject.Curves = CurveData;
	Subject.MetaData = MetaData;
	Subject.Time = Time;

	SendSubjectFrame(SubjectName, Subject);
}

bool FLiveLinkProvider::UpdateSubjectFrameData(const FName SubjectName, FLiveLinkFrameDataStruct&& FrameData, const TMap<FName, FString>& ExtraAnnotations)
{
	FScopeLock Lock(&CriticalSection);

	if (SubjectName == NAME_None)
	{
		return false;
	}

	FTrackedStaticData* StaticData = GetLastSubjectStaticData(SubjectName);
	if (StaticData == nullptr)
	{
		return false;
	}

	UClass* RoleClass = StaticData->RoleClass.Get();
	if (RoleClass == nullptr)
	{
		return false;
	}

	if (RoleClass->GetDefaultObject<ULiveLinkRole>()->GetFrameDataStruct() != FrameData.GetStruct())
	{
		return false;
	}

	// Assign a FrameId to this subject

	if (FLiveLinkBaseFrameData* BaseData = FrameData.GetBaseData())
	{
		int32& NextFrameId = SubjectFrameIds.FindOrAdd(SubjectName, 0);

		if (BaseData->FrameId < 0)
		{
			BaseData->FrameId = NextFrameId;
			NextFrameId = int32(uint32(NextFrameId) + 1u);
		}
		else
		{
			// Provider manually set FrameId, so honor it and update our internal one with it.
			NextFrameId = FMath::Max(NextFrameId, int32(uint32(BaseData->FrameId) + 1u));
		}
	}

	ValidateConnections();

	TMap<FName, FString> Annotations;
	Annotations.Add(FLiveLinkMessageAnnotation::SubjectAnnotation, SubjectName.ToString());
	Annotations.Append(ExtraAnnotations);

	if (ConnectedAddresses.Num() > 0)
	{
		TArray<FMessageAddress> Addresses;
		GetFilteredAddresses(SubjectName, Addresses);

		bool bCanCompressData = CVarLiveLinkCompressAnimData.GetValueOnAnyThread() && FrameData.GetStruct() == FLiveLinkAnimationFrameData::StaticStruct();
		TMap<FMessageAddress, bool> SupportsCompression;

		bool bAnyClientSupportsCompression = false;
		for (const FMessageAddress& Address : Addresses)
		{
			FTrackedAddress* TrackedAddress = Algo::FindBy(ConnectedAddresses, Address, &FTrackedAddress::Address);
			check(TrackedAddress);
			const bool bSupportsCompression = TrackedAddress->bSupportsCompressedPayloads;
			SupportsCompression.Add(Address, bSupportsCompression);

			bAnyClientSupportsCompression = bAnyClientSupportsCompression || bSupportsCompression;
		}

		bCanCompressData = bCanCompressData && bAnyClientSupportsCompression;

		FLiveLinkSerializedFrameData SerializedData;
		if (bCanCompressData && FrameData.GetStruct() == FLiveLinkAnimationFrameData::StaticStruct())
		{
			// If this is animation data, serialize and compress it to reduce throughput.
			SerializedData = SerializeAnimationData(FrameData.Cast<FLiveLinkAnimationFrameData>());
		}
	
		for (const FMessageAddress& Address : Addresses)
		{
			const bool bClientSupportsCompression = bCanCompressData && SupportsCompression.FindChecked(Address);

			if (bClientSupportsCompression)
			{
				FLiveLinkSerializedFrameData* SerializedFrameMessage = FMessageEndpoint::MakeMessage<FLiveLinkSerializedFrameData>(SerializedData);
				MessageEndpoint->Send(SerializedFrameMessage, FLiveLinkSerializedFrameData::StaticStruct(), EMessageFlags::None, Annotations, nullptr, { Address }, FTimespan::Zero(), FDateTime::MaxValue());
			}
			else
			{
				MessageEndpoint->Send(FrameData.CloneData(), const_cast<UScriptStruct*>(FrameData.GetStruct()), EMessageFlags::None, Annotations, nullptr, { Address }, FTimespan::Zero(), FDateTime::MaxValue());
			}
		}
	}

	SetLastSubjectFrameData(SubjectName, MoveTemp(FrameData), MoveTemp(Annotations));

	return true;
}

FLiveLinkSerializedFrameData FLiveLinkProvider::SerializeAnimationData(FLiveLinkAnimationFrameData* AnimData)
{
	FLiveLinkSerializedFrameData SerializedFrameData;

	// Convert double FTransform to float FTransform to further reduce payload size.
	FLiveLinkFloatAnimationFrameData FloatTransforms = FLiveLinkFloatAnimationFrameData::FromAnimData(*AnimData);
	SerializedFrameData.SetTypedPayload(FloatTransforms, ELiveLinkPayloadCompressionType::Always);

	return SerializedFrameData;
}

int32 FLiveLinkProvider::GetSupportedLiveLinkVersion()
{
	return LIVELINK_SupportedVersion;
}

bool FLiveLinkProvider::HasConnection() const
{
	FScopeLock Lock(&CriticalSection);

	FConnectionValidator Validator;

	for (const FTrackedAddress& Connection : ConnectedAddresses)
	{
		if (Validator(Connection))
		{
			return true;
		}
	}
	return false;
}

FDelegateHandle FLiveLinkProvider::RegisterConnStatusChangedHandle(const FLiveLinkProviderConnectionStatusChanged::FDelegate& ConnStatusChanged)
{
	return OnConnectionStatusChanged.Add(ConnStatusChanged);
}

void FLiveLinkProvider::UnregisterConnStatusChangedHandle(FDelegateHandle Handle)
{
	OnConnectionStatusChanged.Remove(Handle);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FLiveLinkProvider::HandlePingMessage(const FLiveLinkPingMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (Message.LiveLinkVersion < LIVELINK_SupportedVersion)
	{
		UE_LOGF(LogLiveLinkMessageBus, Warning, "A unsupported version of LiveLink is trying to communicate. Requested version: '%d'. Supported version: '%d'.", Message.LiveLinkVersion, LIVELINK_SupportedVersion)
		return;
	}

	MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FLiveLinkPongMessage>(ProviderName, MachineName, Message.PollRequest, LIVELINK_SupportedVersion), GetAnnotations(), Context->GetSender());
}

void FLiveLinkProvider::HandleConnectMessage(const FLiveLinkConnectMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	FScopeLock Lock(&CriticalSection);

	if (Message.LiveLinkVersion < LIVELINK_SupportedVersion)
	{
		UE_LOGF(LogLiveLinkMessageBus, Error, "A unsupported version of LiveLink is trying to connect. Requested version: '%d'. Supported version: '%d'.", Message.LiveLinkVersion, LIVELINK_SupportedVersion)
		return;
	}

	const FMessageAddress& ConnectionAddress = Context->GetSender();

	if (!ConnectedAddresses.ContainsByPredicate([=](const FTrackedAddress& Address) { return Address.Address == ConnectionAddress; }))
	{
		// Keep track of what clients supports compressed payloads to avoid sending them data they can't interpret.
		const bool bSupportsCompressedPayloads = Context->GetAnnotations().Contains(FLiveLinkMessageAnnotation::CompressedPayloadSupport);
		ConnectedAddresses.Add(FTrackedAddress(ConnectionAddress, bSupportsCompressedPayloads));

		// LiveLink version 1 path
		for (const auto& Subject : Subjects)
		{
			SendSubject(Subject.Key, Subject.Value);
			FPlatformProcess::Sleep(0.1f); //HACK: Try to help these go in order, editor needs extra buffering support to make sure this isn't needed in future.
			SendSubjectFrame(Subject.Key, Subject.Value);
		}

		// LiveLink version 2 path
		TArray<FMessageAddress> MessageAddress;
		MessageAddress.Add(ConnectionAddress);

		TMap<FName, FString> Annotations = GetAnnotations();
		Annotations.Add(FLiveLinkMessageAnnotation::SubjectAnnotation, TEXT(""));
		Annotations.Add(FLiveLinkMessageAnnotation::RoleAnnotation, TEXT(""));

		for (const FTrackedStaticData& Data : StaticDatas)
		{
			UClass* RoleClass = Data.RoleClass.Get();
			Annotations.Append(Data.Annotations);

			MessageEndpoint->Send(Data.StaticData.CloneData(), const_cast<UScriptStruct*>(Data.StaticData.GetStruct()), EMessageFlags::Reliable, Annotations, nullptr, MessageAddress, FTimespan::Zero(), FDateTime::MaxValue());
		}

		FPlatformProcess::Sleep(0.1f); //HACK: Try to help these go in order, editor needs extra buffering support to make sure this isn't needed in future.

		for (const FTrackedFrameData& Data : FrameDatas)
		{
			Annotations.Append(Data.Annotations);
			MessageEndpoint->Send(Data.FrameData.CloneData(), const_cast<UScriptStruct*>(Data.FrameData.GetStruct()), EMessageFlags::None, Annotations, nullptr, MessageAddress, FTimespan::Zero(), FDateTime::MaxValue());
		}

		OnConnectionStatusChanged.Broadcast();
	}
}

void FLiveLinkProvider::HandleHeartbeat(const FLiveLinkHeartbeatMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	FScopeLock Lock(&CriticalSection);

	FTrackedAddress* TrackedAddress = ConnectedAddresses.FindByPredicate([=](const FTrackedAddress& ConAddress) { return ConAddress.Address == Context->GetSender(); });
	if (TrackedAddress)
	{
		TrackedAddress->LastHeartbeatTime = FPlatformTime::Seconds();

		// Respond so editor gets heartbeat too
		MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FLiveLinkHeartbeatMessage>(), GetAnnotations(), Context->GetSender());
	}
}

TSharedPtr<ILiveLinkProvider> ILiveLinkProvider::CreateLiveLinkProvider(const FString& ProviderName)
{
	return MakeShareable(new FLiveLinkProvider(ProviderName));
}

void FLiveLinkProvider::CreateMessageEndpoint(FMessageEndpointBuilder& EndpointBuilder)
{
	MessageEndpoint = EndpointBuilder
		.ReceivingOnAnyThread()
		.Handling<FLiveLinkPingMessage>(this, &FLiveLinkProvider::HandlePingMessage)
		.Handling<FLiveLinkConnectMessage>(this, &FLiveLinkProvider::HandleConnectMessage)
		.Handling<FLiveLinkHeartbeatMessage>(this, &FLiveLinkProvider::HandleHeartbeat)
		.NotificationHandling(FOnBusNotification::CreateRaw(this, &FLiveLinkProvider::OnMessageBusNotification));


	Subscribe<FLiveLinkPingMessage>();
}

void FLiveLinkProvider::GetConnectedAddresses(TArray<FMessageAddress>& Addresses)
{
	ValidateConnections();
	Addresses.Reserve(ConnectedAddresses.Num());
	for (const FTrackedAddress& Address : ConnectedAddresses)
	{
		Addresses.Add(Address.Address);
	}
}

void FLiveLinkProvider::GetFilteredAddresses(FName SubjectName, TArray<FMessageAddress>& Addresses)
{
	ValidateConnections();
	Addresses.Reserve(ConnectedAddresses.Num());

	Algo::TransformIf(ConnectedAddresses, Addresses,
		[this, SubjectName](const FTrackedAddress& Address){ return ShouldTransmitToSubject_AnyThread(SubjectName, Address.Address); },
		[](const FTrackedAddress& Address){ return Address.Address; });
}
