// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkTypes.h"

#include "LiveLinkCompression.generated.h"

#define UE_API LIVELINKMESSAGEBUSFRAMEWORK_API

/** Utility struct to serialize a byte array. */
USTRUCT()
struct FLiveLinkByteArray
{
	GENERATED_BODY()

	bool Serialize(FArchive& Ar)
	{
		int32 Num = Bytes.Num();
		Ar << Num;
		if (Ar.IsLoading())
		{
			Bytes.AddUninitialized(Num);
		}
		Ar.Serialize(Bytes.GetData(), Num);
		return true;
	}

	UPROPERTY()
	TArray<uint8> Bytes;
};

template<>
struct TStructOpsTypeTraits<FLiveLinkByteArray> : public TStructOpsTypeTraitsBase2<FLiveLinkByteArray>
{
	enum
	{
		WithSerializer = true,
	};
};

/** What compression method should be used on the LiveLink data. */
UENUM()
enum class ELiveLinkCompressionMethod : uint8
{
	/** Data is uncompressed. */
	Uncompressed,

	/** Oodle compression is used for the serialized data. */
	Oodle,

	/**  ZLib compression is used for the serialized data. */
	Zlib
};

/** The bias (if any) to use when compressing the data. */
UENUM()
enum class ELiveLinkCompressionBias : uint8
{
	/** Compress without a bias. */
	None,

	/** Compress with a bias for size */
	Size,

	/** Compress with a bias for speed. */
	Speed
};

namespace UE::LiveLink::Compression
{

	/** Get the compression type from the console variable setting. */
	int32 GetConsoleVariableCompressionType();

	/** Function to check the console variable for the compression flags. Speed vs size. */
	int32 GetConsoleVariableCompressionFlags();

	/** Based on the size of the data to compress indicate if we should invoke the compressor. */
	template <typename SizeType>
	inline bool ShouldCompress(SizeType DataSize)
	{
		static_assert(std::is_integral<SizeType>::value);
		const int32 CompressionLimit = 32;
		const int32 MinLimit = 512;
		if (CompressionLimit <= 0)
		{
			return DataSize > MinLimit;
		}
		const SizeType MaxPackageDataSizeForCompression = static_cast<SizeType>(CompressionLimit) * 1024 * 1024;
		return DataSize > MinLimit && DataSize <= MaxPackageDataSizeForCompression;
	}

	/** Get the named compression algorithm to invoke with serializer and memory compressors. */
	inline FName GetCompressionAlgorithm()
	{
		const int32 CompressionAlgo = GetConsoleVariableCompressionType();
		if (CompressionAlgo == 1)
		{
			return NAME_Oodle;
		}
		return NAME_Zlib;
	}

	/** Get the default flags to use when invoking the compressor. */
	inline ECompressionFlags GetCompressionFlags()
	{
		const int32 CompressionFlags = GetConsoleVariableCompressionFlags();
		if (CompressionFlags == 0)
		{
			return COMPRESS_NoFlags;
		}
		if (CompressionFlags == 1)
		{
			return COMPRESS_BiasSize;
		}
		return COMPRESS_BiasSpeed;
	}

	/** Get the named compression algorithm based on the provided details.  Current this is only Zlib or Oodle. */
	inline FName GetCompressionAlgorithm(ELiveLinkCompressionMethod Method)
	{
		return Method == ELiveLinkCompressionMethod::Oodle ? NAME_Oodle : NAME_Zlib;
	}

	/** Get the compression enum value given the LiveLink compression settings.  This the value passed into the serializer and memory compressor. */
	inline ECompressionFlags GetCoreCompressionFlags(ELiveLinkCompressionBias InBias)
	{
		if (InBias == ELiveLinkCompressionBias::Size)
		{
			return COMPRESS_BiasSize;
		}
		else if (InBias == ELiveLinkCompressionBias::Speed)
		{
			return COMPRESS_BiasSpeed;
		}
		return COMPRESS_NoFlags;
	}

	/** Get LiveLinkCompressionBias from a ECompressionFlags. */
	inline ELiveLinkCompressionBias GetCompressionBias(ECompressionFlags Flags)
	{
		if (Flags == COMPRESS_BiasMemory)
		{
			return ELiveLinkCompressionBias::Size;
		}
		else if (Flags == COMPRESS_BiasSpeed)
		{
			return ELiveLinkCompressionBias::Speed;
		}
		else
		{
			return ELiveLinkCompressionBias::None;
		}
	}

	/** Convert a compression method name to a ELiveLinkCompressionMethod. */
	inline ELiveLinkCompressionMethod GetCompressionMethod(FName NamedMethod)
	{
		check(NamedMethod == NAME_Oodle || NamedMethod == NAME_Zlib);
		if (NamedMethod == NAME_Oodle)
		{
			return ELiveLinkCompressionMethod::Oodle;
		}
		else
		{
			return ELiveLinkCompressionMethod::Zlib;
		}
	}
}

/** What method should be used for serialziing. */
UENUM()
enum class ELiveLinkPayloadSerializationMethod : uint8
{
	// The data will be serialized using standard platform method.
	Standard = 0,
	// The data will be serialized using Cbor method.
	Cbor,
};

/** Dictates when compression should be used. */
UENUM()
enum class ELiveLinkPayloadCompressionType : uint8
{
	// The serialized data will not be compressed.
	None = 0,
	// The serialized data will be compressed based on struct size.
	Heuristic,
	// The serialized data will always be compressed.
	Always
};

/** Utility struct used to wrap serialized data and handle compression / decompression. */
USTRUCT()
struct FLiveLinkSerializedFrameData
{
	GENERATED_BODY()

	FLiveLinkSerializedFrameData() = default;

	FLiveLinkSerializedFrameData(ELiveLinkPayloadSerializationMethod SerializeMethod)
		: SerializationMethod(SerializeMethod)
	{
	}

	/** Initialize this payload from the given data */
	UE_API bool SetPayload(const FStructOnScope& InPayload, ELiveLinkPayloadCompressionType CompressionType = ELiveLinkPayloadCompressionType::Heuristic);
	UE_API bool SetPayload(const UScriptStruct* InPayloadType, const void* InPayloadData, ELiveLinkPayloadCompressionType CompressionType = ELiveLinkPayloadCompressionType::Heuristic);

	template <typename T>
	bool SetTypedPayload(const T& InPayloadData, ELiveLinkPayloadCompressionType CompressType = ELiveLinkPayloadCompressionType::Heuristic)
	{
		return SetPayload(TBaseStructure<T>::Get(), &InPayloadData, CompressType);
	}

	/** Extract the payload into an in-memory instance */
	UE_API bool GetPayload(FStructOnScope& OutPayload) const;
	UE_API bool GetPayload(const UScriptStruct* InPayloadType, void* InOutPayloadData) const;

	UE_API bool IsTypeChildOf(const UScriptStruct* InPayloadType) const;

	template<typename T>
	bool IsTypeChildOf() const
	{
		return IsTypeChildOf(TBaseStructure<T>::Get());
	}

	template <typename T>
	bool GetTypedPayload(T& OutPayloadData) const
	{
		return GetPayload(TBaseStructure<T>::Get(), &OutPayloadData);
	}

	/** The typename of the user-defined payload. */
	UPROPERTY(VisibleAnywhere, Category = "Payload")
	FName PayloadTypeName;

	/** Specifies the serialization method used to pack the data */
	UPROPERTY(VisibleAnywhere, Category = "Payload")
	ELiveLinkPayloadSerializationMethod SerializationMethod = ELiveLinkPayloadSerializationMethod::Standard;

	UPROPERTY(VisibleAnywhere, Category = "Payload")
	ELiveLinkCompressionMethod CompressionMethod = ELiveLinkCompressionMethod::Uncompressed;

	UPROPERTY(VisibleAnywhere, Category = "Payload")
	ELiveLinkCompressionBias CompressionBias = ELiveLinkCompressionBias::None;

	/** The uncompressed size of the user-defined payload data. */
	UPROPERTY(VisibleAnywhere, Category = "Payload")
	int32 PayloadSize = 0;

	/** The data of the user-defined payload (potentially stored as compressed binary for compact transfer). */
	UPROPERTY()
	FLiveLinkByteArray PayloadBytes;
};

#undef UE_API
