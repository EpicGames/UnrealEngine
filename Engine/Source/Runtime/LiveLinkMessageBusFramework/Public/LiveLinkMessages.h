// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/Object.h"
#include "LiveLinkRefSkeleton.h"
#include "LiveLinkTypes.h"
#include "LiveLinkMessages.generated.h"

#define UE_API LIVELINKMESSAGEBUSFRAMEWORK_API

struct FLiveLinkMessageAnnotation
{
	static UE_API FName SubjectAnnotation;
	static UE_API FName RoleAnnotation;
	static UE_API FName OriginalSourceAnnotation;
	/** Indicates what mode this instance is running in. */
	static UE_API FName TopologyModeAnnotation;
	/** Presense of this annotation informs providers that this client supports compressed animation payloads. */
	static UE_API FName CompressedPayloadSupport;
};

UENUM()
enum class ELiveLinkTopologyMode : uint8
{
	Hub   			UMETA(ToolTip = "LiveLinkHub instance that can receive data from other hub instances and transmit it to Unreal Editor instances."),
	Spoke 			UMETA(ToolTip = "LiveLinkHub instance can only transmit data to other LiveLinkHub instances."),
	UnrealClient 	UMETA(ToolTip = "Unreal Editor instance that can only receive data from LiveLinkHub"),
	External 		UMETA(ToolTip = "Client that can transmit to Hubs, Spokes and UnrealClients. (Default mode for message bus providers)")
};


namespace UE::LiveLink::Messaging
{
	/** Whether the left hand side mode can receive from the right hand side mode LiveLink Source. */
	LIVELINKMESSAGEBUSFRAMEWORK_API bool CanReceiveFrom(ELiveLinkTopologyMode LHSMode, ELiveLinkTopologyMode RHSMode);

	/** Whether the left hand side mode can transmit to the right hand side mode Client. */
	LIVELINKMESSAGEBUSFRAMEWORK_API bool CanTransmitTo(ELiveLinkTopologyMode LHSMode, ELiveLinkTopologyMode RHSMode);
}

USTRUCT()
struct FLiveLinkPingMessage
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid PollRequest;

	UPROPERTY()
	int32 LiveLinkVersion = 1;

	// V1: 5.6 and earlier, V2: 5.7 and later.
	UPROPERTY()
	int32 DiscoveryProtocolVersion = 1;

	// default constructor for the receiver
	FLiveLinkPingMessage() = default;

	FLiveLinkPingMessage(const FGuid& InPollRequest, int32 InLiveLinkVersion) 
		: PollRequest(InPollRequest)
		, LiveLinkVersion(InLiveLinkVersion)
		, DiscoveryProtocolVersion(2) // Note: New ping messages use DiscoveryProtocol V2
	{}
};

USTRUCT()
struct FLiveLinkPongMessage
{
	GENERATED_BODY()

	UPROPERTY()
	FString ProviderName;

	UPROPERTY()
	FString MachineName;

	UPROPERTY()
	FGuid PollRequest;

	UPROPERTY()
	int32 LiveLinkVersion = 1;

	// V1: 5.6 and earlier, V2: 5.7 and later.
	UPROPERTY()
	int32 DiscoveryProtocolVersion = 1;

	UPROPERTY()
	double CreationPlatformTime = -1.0;

	// default constructor for the receiver
	FLiveLinkPongMessage() = default;

	UE_DEPRECATED(5.0, "This version of the FLiveLinkPongMessage constructor is deprecated. Please use the new constructor instead to ensure the LiveLinkVersion is set properly.")
	FLiveLinkPongMessage(const FString& InProviderName, const FString& InMachineName, const FGuid& InPollRequest) : ProviderName(InProviderName), MachineName(InMachineName), PollRequest(InPollRequest), DiscoveryProtocolVersion(2), CreationPlatformTime(FPlatformTime::Seconds()) {}

	FLiveLinkPongMessage(const FString& InProviderName, const FString& InMachineName, const FGuid& InPollRequest, int32 InLiveLinkVersion) : ProviderName(InProviderName), MachineName(InMachineName), PollRequest(InPollRequest), LiveLinkVersion(InLiveLinkVersion), DiscoveryProtocolVersion(2), CreationPlatformTime(FPlatformTime::Seconds()) {}
};

USTRUCT()
struct FLiveLinkConnectMessage
{
	GENERATED_BODY()

	UPROPERTY()
	int32 LiveLinkVersion = 1;
};

USTRUCT()
struct FLiveLinkHeartbeatMessage
{
	GENERATED_BODY()
};

USTRUCT()
struct FLiveLinkClearSubject
{
	GENERATED_BODY()

	// Name of the subject to clear
	UPROPERTY()
	FName SubjectName;

	FLiveLinkClearSubject() {}
	FLiveLinkClearSubject(const FName& InSubjectName) : SubjectName(InSubjectName) {}
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
USTRUCT()
struct UE_DEPRECATED(4.23, "FLiveLinkSubjectDataMessage is deprecated. Please use the LiveLink animation role.") FLiveLinkSubjectDataMessage
{
	GENERATED_BODY()

	UPROPERTY()
	FLiveLinkRefSkeleton RefSkeleton;

	UPROPERTY()
	FName SubjectName;
};

USTRUCT()
struct UE_DEPRECATED(4.23, "FLiveLinkSubjectDataMessage is deprecated. Please use the LiveLink animation role.") FLiveLinkSubjectFrameMessage
{
	GENERATED_BODY()

	UPROPERTY()
	FName SubjectName;

	// Bone Transform data for this frame
	UPROPERTY()
	TArray<FTransform> Transforms;

	// Curve data for this frame
	UPROPERTY()
	TArray<FLiveLinkCurveElement> Curves;

	// Subject MetaData for this frame
	UPROPERTY()
	FLiveLinkMetaData MetaData;

	// Incrementing time for interpolation
	UPROPERTY()
	double Time = 0.0;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API
