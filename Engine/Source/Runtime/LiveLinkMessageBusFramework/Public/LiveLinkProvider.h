// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "LiveLinkRole.h"
#include "LiveLinkTypes.h"
#include "Math/Transform.h"
#include "MessageEndpointBuilder.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

class ULiveLinkRole;

/** Delegate called when the connection status of the provider has changed. */
DECLARE_MULTICAST_DELEGATE(FLiveLinkProviderConnectionStatusChanged);

struct ILiveLinkProvider
{
public:
	LIVELINKMESSAGEBUSFRAMEWORK_API static TSharedPtr<ILiveLinkProvider> CreateLiveLinkProvider(const FString& ProviderName);

	/**
	 * Create a Live Link Provider based on a class derived from ILiveLinkProvider instead of using the default Live Link Provider.
	 * @param ProviderName		The provider name.
	 * @param EndpointBuilder	An endpoint builder that can be used to add additional message handlers.
	 * @return					Shared pointer to the ILiveLinkProvider-derived class. Use StaticCastSharedPtr to cast it back to child class type.
	 */
	template<typename T>
	static TSharedPtr<ILiveLinkProvider> CreateLiveLinkProvider(const FString& ProviderName,
																struct FMessageEndpointBuilder&& EndpointBuilder)
	{
		return MakeShared<T>(ProviderName, MoveTemp(EndpointBuilder));
	}

	virtual ~ILiveLinkProvider() = default;

	/**
	 * Send, to UE, the static data of a subject.
	 * @param SubjectName	    The name of the subject
	 * @param Role			    The Live Link role of the subject. The StaticData type should match the role's data.
	 * @param StaticData	    The static data of the subject.
							    The FLiveLinkStaticDataStruct doesn't have a copy constructor.
	 *						    The argument is passed by r-value to help the user understand the compiler error message.
	 * @param ExtraAnnotations	Extra annotations to pass in with the static data message.
	 * @return				True if the message was sent or is pending an active connection.
	 * @see					UpdateSubjectFrameData, RemoveSubject
	 */
	virtual bool UpdateSubjectStaticData(const FName SubjectName, TSubclassOf<ULiveLinkRole> Role, FLiveLinkStaticDataStruct&& StaticData, const TMap<FName, FString>& ExtraAnnotations = {}) = 0;

	/**
	 * Inform UE that a subject won't be streamed anymore.
	 * @param SubjectName	The name of the subject.
	 */
	virtual void RemoveSubject(const FName SubjectName) = 0;

	/**
	 * Send the static data of a subject to UE.
	 * @param SubjectName		The name of the subject
	 * @param StaticData		The frame data of the subject. The type should match the role's data send with UpdateSubjectStaticData.
								The FLiveLinkFrameDataStruct doesn't have a copy constructor.
	 *							The argument is passed by r-value to help the user understand the compiler error message.
	 * @param ExtraAnnotations	Extra annotations to pass in with the frame data message.
	 * @return					True if the message was sent or is pending an active connection.
	 * @see						UpdateSubjectStaticData, RemoveSubject
	 */
	virtual bool UpdateSubjectFrameData(const FName SubjectName, FLiveLinkFrameDataStruct&& FrameData, const TMap<FName, FString>& ExtraAnnotations = {}) = 0;

	/** Is this provider currently connected to something. */
	virtual bool HasConnection() const = 0;

	/** Function for managing connection status changed delegate. */
	virtual FDelegateHandle RegisterConnStatusChangedHandle(const FLiveLinkProviderConnectionStatusChanged::FDelegate& ConnStatusChanged) = 0;

	/** Function for managing connection status changed delegate. */
	virtual void UnregisterConnStatusChangedHandle(FDelegateHandle Handle) = 0;
};
