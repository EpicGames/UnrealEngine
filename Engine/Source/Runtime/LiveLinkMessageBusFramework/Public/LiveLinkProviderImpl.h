// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMessageContext.h"
#include "LiveLinkMessages.h"
#include "LiveLinkProvider.h"
#include "MessageEndpoint.h"

#define UE_API LIVELINKMESSAGEBUSFRAMEWORK_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS
// Subject that the application has told us about
struct FTrackedSubject
{
	// Ref skeleton to go with transform data
	FLiveLinkRefSkeleton RefSkeleton;

	// Bone transform data
	TArray<FTransform> Transforms;

	// Curve data
	TArray<FLiveLinkCurveElement> Curves;

	// MetaData for subject
	FLiveLinkMetaData MetaData;

	// Incrementing time (application time) for interpolation purposes
	double Time;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

struct FLiveLinkProvider : public ILiveLinkProvider
{
private:
	const FString ProviderName;
	const FString MachineName;

	TSharedPtr<class FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	// Lock to stop multiple threads accessing the CurrentPreset at the same time
	mutable FCriticalSection CriticalSection;

	// Array of our current connections
	TArray<struct FTrackedAddress> ConnectedAddresses;

	// Cache of our current subject state
	TArray<struct FTrackedStaticData> StaticDatas;
	TArray<struct FTrackedFrameData> FrameDatas;
	TMap<FName, FTrackedSubject> Subjects;

	// Per-subject FrameIds
	TMap<FName, int32> SubjectFrameIds;

	// Delegate to notify interested parties when the client sources have changed
	FLiveLinkProviderConnectionStatusChanged OnConnectionStatusChanged;

private:

	// Begin message bus message handlers

	// Handles ping message and sends back a pong
	void HandlePingMessage(const FLiveLinkPingMessage& Message, const TSharedRef<class IMessageContext, ESPMode::ThreadSafe>& Context);

	void HandleHeartbeat(const FLiveLinkHeartbeatMessage& Message,
						 const TSharedRef<class IMessageContext, ESPMode::ThreadSafe>& Context);
	// End message bus message handlers

	FTrackedSubject& GetTrackedSubject(const FName& SubjectName);

	void SendSubject(FName SubjectName, const struct FTrackedSubject& Subject);

	void SendSubjectFrame(FName SubjectName, const struct FTrackedSubject& Subject);

	// Get the cached data for the named subject
	FTrackedStaticData* GetLastSubjectStaticData(const FName& SubjectName);

	FTrackedFrameData* GetLastSubjectFrameData(const FName& SubjectName);

	void SetLastSubjectStaticData(FName SubjectName, TSubclassOf<ULiveLinkRole> Role, FLiveLinkStaticDataStruct&& StaticData, TMap<FName, FString>&& Annotations);

	void SetLastSubjectFrameData(FName SubjectName, FLiveLinkFrameDataStruct&& FrameData, TMap<FName, FString>&& SubjectAnnotations);

	// Clear a existing track subject
	void ClearTrackedSubject(const FName& SubjectName);

	// Get the connected addresses that should receive livelink data.
	void GetFilteredAddresses(FName SubjectName, TArray<FMessageAddress>& Addresses);

	// Serialize animation frame data into a compressed payload to reduce throughput.
	struct FLiveLinkSerializedFrameData SerializeAnimationData(struct FLiveLinkAnimationFrameData* AnimData);

protected:
	// Get the minimum supported version of livelink clients that can connect to this provider.
	UE_API static int32 GetSupportedLiveLinkVersion();

	// Update connected addresses and send information to the connected source
	UE_API void HandleConnectMessage(const FLiveLinkConnectMessage& Message, const TSharedRef<class IMessageContext, ESPMode::ThreadSafe>& Context);

	// Create the message bus message endoing responsble for dispatching message bus messages to their respective handlers
	UE_API void CreateMessageEndpoint(struct FMessageEndpointBuilder& EndpointBuilder);
	
	// Get the addresses of all connected instances.
	UE_API void GetConnectedAddresses(TArray<FMessageAddress>& Addresses);

	// Validate our current connections, removing those that have timed out.
	UE_API void ValidateConnections();

	// Close a connection using its address.
	UE_API void CloseConnection(FMessageAddress Address);

	// Get the cached data struct for a subject
	UE_API TPair<UClass*, FLiveLinkStaticDataStruct*> GetLastSubjectStaticDataStruct(FName SubjectName);

	// Invoked by MessageBus when a node is registered or unregistered.
	virtual void OnMessageBusNotification(const FMessageBusNotification& Notification) {}

	// Send message (only to connected addresses)
	template<typename MessageType>
	void SendMessage(MessageType* Message, EMessageFlags Flags = EMessageFlags::None)
	{
		if (!Message)
		{
			return;
		}

		TArray<FMessageAddress> Addresses;
		GetConnectedAddresses(Addresses);
		if (Addresses.Num() != 0)
		{
			MessageEndpoint->Send(Message, Flags, {}, nullptr, Addresses, FTimespan::Zero(), FDateTime::MaxValue());
		}
	}

	// Send message to a specific address.
	template<typename MessageType>
	void SendMessage(MessageType* Message, const FMessageAddress& Address, EMessageFlags Flags = EMessageFlags::None, const TMap<FName, FString>& Annotations = {})
	{
		if (!Message || !Address.IsValid())
		{
			return;
		}

		MessageEndpoint->Send(Message, Flags, Annotations, nullptr, { Address }, FTimespan::Zero(), FDateTime::MaxValue());
	}

	// Send message to a list of addresses.
	template<typename MessageType>
	void SendMessage(MessageType* Message, const TArray<FMessageAddress>& Addresses, EMessageFlags Flags = EMessageFlags::None, const TMap<FName, FString>& Annotations = {})
	{
		if (!Message || !Addresses.Num())
		{
			return;
		}

		MessageEndpoint->Send(Message, Flags, Annotations, nullptr, Addresses, FTimespan::Zero(), FDateTime::MaxValue());
	}

	// Broadcast message on the network.
	template<typename MessageType>
	void Publish(MessageType* Message)
	{
		if (!Message)
		{
			return;
		}

		MessageEndpoint->Publish(Message);
	}

	// Broadcast message on the network with annotations.
	template<typename MessageType>
	void Publish(MessageType* Message, const TMap<FName, FString>& Annotations)
	{
		if (!Message)
		{
			return;
		}

		MessageEndpoint->Publish(Message, Annotations);
	}

	// Subscribe to a message that's been published.
	template<typename MessageType>
	void Subscribe()
	{
		if (MessageEndpoint.IsValid())
		{
			MessageEndpoint->Subscribe<MessageType>();
		}
	}

	// Get the name of this provider.
	const FString& GetProviderName() const
	{
		return ProviderName;
	}
	
	// Get the name of this machine.
	const FString& GetMachineName() const
	{
		return MachineName;
	}

	// Called after ValidateConnections removes invalid connections 
	virtual void OnConnectionsClosed(const TArray<FMessageAddress>& ClosedAddresses) {}

	// Get annotations to include on every message sent by this provider
	virtual TMap<FName, FString> GetAnnotations() const
	{
#if WITH_ENGINE
		// When running a game/editor, mark this provider as an unreal client.
		FString TopologyModeValue = StaticEnum<ELiveLinkTopologyMode>()->GetNameStringByValue(static_cast<int64>(ELiveLinkTopologyMode::UnrealClient));
		return { {FLiveLinkMessageAnnotation::TopologyModeAnnotation, MoveTemp(TopologyModeValue)} };
#endif
		return {};
	}

	// Get whether a combination of a subject/client should receive livelink data.
	virtual bool ShouldTransmitToSubject_AnyThread(FName SubjectName, FMessageAddress Address) const
	{
		return true;
	}

	// Send a clear subject message to indicate that the subject should be removed from the connected client.
	UE_API void SendClearSubjectToConnections(FName SubjectName);

	// Get the MessageBus address of the provider.
	UE_API FMessageAddress GetEndpointAddress() const;

	// Constructor for derived classes that allows specifying that no endpoint should be created.
	UE_API FLiveLinkProvider(const FString& InProviderName, bool bInCreateEndpoint);

public:
	UE_API FLiveLinkProvider(const FString& InProviderName);

	UE_API FLiveLinkProvider(const FString& InProviderName, struct FMessageEndpointBuilder&& EndpointBuilder);

	UE_API virtual ~FLiveLinkProvider() override;

	UE_API virtual void UpdateSubject(const FName& SubjectName, const TArray<FName>& BoneNames, const TArray<int32>& BoneParents);

	UE_API virtual bool UpdateSubjectStaticData(const FName SubjectName, TSubclassOf<ULiveLinkRole> Role, FLiveLinkStaticDataStruct&& StaticData, const TMap<FName, FString>& ExtraAnnotations = {}) override;

	UE_API virtual void ClearSubject(const FName& SubjectName);

	UE_API virtual void RemoveSubject(const FName SubjectName) override;

UE_API PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual void UpdateSubjectFrame(const FName& SubjectName, const TArray<FTransform>& BoneTransforms, const TArray<FLiveLinkCurveElement>& CurveData, double Time);

	UE_API virtual void UpdateSubjectFrame(const FName& SubjectName, const TArray<FTransform>& BoneTransforms, const TArray<FLiveLinkCurveElement>& CurveData,
									const FLiveLinkMetaData& MetaData, double Time);
UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual bool UpdateSubjectFrameData(const FName SubjectName, FLiveLinkFrameDataStruct&& FrameData, const TMap<FName, FString>& ExtraAnnotations = {}) override;

	UE_API virtual bool HasConnection() const override;

	UE_API virtual FDelegateHandle RegisterConnStatusChangedHandle(const FLiveLinkProviderConnectionStatusChanged::FDelegate& ConnStatusChanged) override;

	UE_API virtual void UnregisterConnStatusChangedHandle(FDelegateHandle Handle) override;
};

#undef UE_API
